<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="..\Shared\UnrealEngine.csproj.props" />

  <PropertyGroup>
    <ImplicitUsings>enable</ImplicitUsings>
    <Nullable>enable</Nullable>

    <IsPackable>false</IsPackable>
    <IsTestProject>true</IsTestProject>
  </PropertyGroup>

  <ItemGroup>
    <PackageReference Include="Microsoft.NET.Test.Sdk" Version="18.4.0" />
    <PackageReference Include="MSTest.TestAdapter" Version="4.2.1" />
    <PackageReference Include="MSTest.TestFramework" Version="4.2.1" />
    <PackageReference Include="coverlet.collector" Version="8.0.1">
      <PrivateAssets>all</PrivateAssets>
      <IncludeAssets>runtime; build; native; contentfiles; analyzers; buildtransitive</IncludeAssets>
    </PackageReference>
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\AutomationTool\AutomationUtils\AutomationUtils.Automation.csproj" />
    <ProjectReference Include="..\AutomationTool\Scripts\AutomationScripts.Automation.csproj" />
    <ProjectReference Include="..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\Shared\EpicGames.Perforce\EpicGames.Perforce.csproj" />
    <ProjectReference Include="..\UnrealBuildTool\UnrealBuildTool.csproj" />
  </ItemGroup>

</Project>
