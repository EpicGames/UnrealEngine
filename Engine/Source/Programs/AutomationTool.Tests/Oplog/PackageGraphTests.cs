// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using AutomationScripts.Oplog;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace AutomationTool.Tests.Oplog
{
	/// <summary>
	/// Tests for <see cref="PackageGraph"/> traversal and the
	/// <see cref="PackageGraph.GetExclusiveDependencyClosure"/> dominance algorithm.
	/// Graphs are built directly from <see cref="PackageGraphData"/> via internals access
	/// so we can exercise edge cases (cycles, external referencers) that the production
	/// builder wouldn't otherwise let us construct ergonomically.
	/// </summary>
	[TestClass]
	public sealed class PackageGraphTests
	{
		// ------------------------------------------------------------
		// Helpers
		// ------------------------------------------------------------

		/// <summary>
		/// Build a synthetic graph with <paramref name="nodeCount"/> nodes numbered 0..N-1.
		/// Hard edges are required; runtime edges optional. Reverse edges are populated
		/// automatically to mirror what <see cref="PackageGraphBuilder"/> does.
		/// </summary>
		private static PackageGraph BuildGraph(
			int nodeCount,
			(int From, int To)[] hardEdges,
			(int From, int To)[]? runtimeEdges = null)
		{
			var data = new PackageGraphData(nodeCount);
			data.NodeCount = nodeCount;

			var hardDeps    = new List<int>[nodeCount];
			var hardRefs    = new List<int>[nodeCount];
			var runtimeDeps = new List<int>[nodeCount];
			var runtimeRefs = new List<int>[nodeCount];

			for (int i = 0; i < nodeCount; i++)
			{
				string name = $"/game/n{i}";
				data.Metadata[i]      = new PackageMetadata(name, (ulong)i, 0, false);
				data.IndexById[(ulong)i] = i;
				data.IndexByName[name]   = i;
				data.SoftDeps[i] = Array.Empty<int>();
				data.SoftRefs[i] = Array.Empty<int>();
				hardDeps[i]    = new List<int>();
				hardRefs[i]    = new List<int>();
				runtimeDeps[i] = new List<int>();
				runtimeRefs[i] = new List<int>();
			}

			foreach (var (from, to) in hardEdges)
			{
				hardDeps[from].Add(to);
				hardRefs[to].Add(from);
			}
			if (runtimeEdges != null)
			{
				foreach (var (from, to) in runtimeEdges)
				{
					runtimeDeps[from].Add(to);
					runtimeRefs[to].Add(from);
				}
			}

			for (int i = 0; i < nodeCount; i++)
			{
				data.HardDeps[i]    = hardDeps[i].ToArray();
				data.HardRefs[i]    = hardRefs[i].ToArray();
				data.RuntimeDeps[i] = runtimeDeps[i].ToArray();
				data.RuntimeRefs[i] = runtimeRefs[i].ToArray();
			}

			return new PackageGraph(data);
		}

		private static PackageNode Node(PackageGraph g, int index)
		{
			Assert.IsTrue(g.TryGetNode((ulong)index, out PackageNode n), $"Node {index} not found");
			return n;
		}

		private static HashSet<int> IndicesOf(IEnumerable<PackageNode> nodes) =>
			new HashSet<int>(nodes.Select(n => n.Index));

		private static void AssertSetEquals(IEnumerable<int> expected, IEnumerable<PackageNode> actual, string message = "")
		{
			var actualIndices = IndicesOf(actual);
			var expectedSet   = new HashSet<int>(expected);
			Assert.IsTrue(
				actualIndices.SetEquals(expectedSet),
				$"{message} Expected {{{string.Join(",", expectedSet.OrderBy(x => x))}}}, got {{{string.Join(",", actualIndices.OrderBy(x => x))}}}");
		}

		// ------------------------------------------------------------
		// GetDependencies / GetReferencers
		// ------------------------------------------------------------

		[TestMethod]
		public void GetDependencies_ReturnsHardOnly_WhenRuntimeExcluded()
		{
			// 0 -hard-> 1, 0 -runtime-> 2
			var g = BuildGraph(3,
				hardEdges: new[] { (0, 1) },
				runtimeEdges: new[] { (0, 2) });

			AssertSetEquals(new[] { 1 }, g.GetDependencies(Node(g, 0), includeRuntimeDependencies: false));
		}

		[TestMethod]
		public void GetDependencies_IncludesRuntime_ByDefault()
		{
			var g = BuildGraph(3,
				hardEdges: new[] { (0, 1) },
				runtimeEdges: new[] { (0, 2) });

			AssertSetEquals(new[] { 1, 2 }, g.GetDependencies(Node(g, 0)));
		}

		[TestMethod]
		public void GetReferencers_ReturnsHardOnly_WhenRuntimeExcluded()
		{
			// 1 -hard-> 0,  2 -runtime-> 0
			var g = BuildGraph(3,
				hardEdges: new[] { (1, 0) },
				runtimeEdges: new[] { (2, 0) });

			AssertSetEquals(new[] { 1 }, g.GetReferencers(Node(g, 0), includeRuntimeReferencers: false));
		}

		[TestMethod]
		public void GetReferencers_IncludesRuntime_ByDefault()
		{
			var g = BuildGraph(3,
				hardEdges: new[] { (1, 0) },
				runtimeEdges: new[] { (2, 0) });

			AssertSetEquals(new[] { 1, 2 }, g.GetReferencers(Node(g, 0)));
		}

		[TestMethod]
		public void GetDependencies_OnLeafNode_ReturnsEmpty()
		{
			var g = BuildGraph(2, hardEdges: new[] { (0, 1) });
			Assert.AreEqual(0, g.GetDependencies(Node(g, 1)).Count);
		}

		// ------------------------------------------------------------
		// GetExclusiveDependencyClosure — basics
		// ------------------------------------------------------------

		[TestMethod]
		public void ExclusiveClosure_EmptySeeds_ReturnsEmpty()
		{
			var g = BuildGraph(3, hardEdges: new[] { (0, 1), (1, 2) });
			var result = g.GetExclusiveDependencyClosure(Array.Empty<PackageNode>());
			Assert.AreEqual(0, result.Count);
		}

		[TestMethod]
		public void ExclusiveClosure_SeedWithNoEdges_ReturnsJustSeed()
		{
			var g = BuildGraph(3, hardEdges: Array.Empty<(int, int)>());
			AssertSetEquals(new[] { 0 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		[TestMethod]
		public void ExclusiveClosure_LinearChain_AllIncluded()
		{
			// 0 -> 1 -> 2 -> 3, no external referencers
			var g = BuildGraph(4, hardEdges: new[] { (0, 1), (1, 2), (2, 3) });
			AssertSetEquals(new[] { 0, 1, 2, 3 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		[TestMethod]
		public void ExclusiveClosure_DiamondWithExternalReferencer_ExcludesTaintedBranch()
		{
			// Seed: 0
			// 0 -> 1, 0 -> 2, X(=3) -> 2
			// 1 is exclusive; 2 is reachable from external X so 2 is excluded.
			var g = BuildGraph(4, hardEdges: new[] { (0, 1), (0, 2), (3, 2) });
			AssertSetEquals(new[] { 0, 1 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		[TestMethod]
		public void ExclusiveClosure_TaintPropagatesForward()
		{
			// Seed: 0
			// 0 -> 1, X(=3) -> 1, 1 -> 2
			// 1 is tainted by X, taint forward-propagates to 2.
			var g = BuildGraph(4, hardEdges: new[] { (0, 1), (3, 1), (1, 2) });
			AssertSetEquals(new[] { 0 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		[TestMethod]
		public void ExclusiveClosure_ExternalReferencerOfSeed_DoesNotTaintSeed()
		{
			// Seed: 0;  X(=2) -> 0 (X points at the seed itself), 0 -> 1
			// Seeds are always part of the result, and 1 is reachable only from seed.
			var g = BuildGraph(3, hardEdges: new[] { (2, 0), (0, 1) });
			AssertSetEquals(new[] { 0, 1 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		// ------------------------------------------------------------
		// Cycles
		// ------------------------------------------------------------

		[TestMethod]
		public void ExclusiveClosure_CycleEntirelyInsideR_AllIncluded()
		{
			// 0 -> 1 -> 2 -> 1 (cycle between 1 and 2). No external referencers into R.
			var g = BuildGraph(3, hardEdges: new[] { (0, 1), (1, 2), (2, 1) });
			AssertSetEquals(new[] { 0, 1, 2 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		[TestMethod]
		public void ExclusiveClosure_CycleWithExternalEntry_AllTainted()
		{
			// 0 -> 1, 1 -> 2, 2 -> 1 (cycle), X(=3) -> 2
			// 2 is tainted (external entry), 1 is tainted (reachable from 2 via cycle).
			var g = BuildGraph(4, hardEdges: new[] { (0, 1), (1, 2), (2, 1), (3, 2) });
			AssertSetEquals(new[] { 0 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		// ------------------------------------------------------------
		// Multi-seed and runtime-edge toggling
		// ------------------------------------------------------------

		[TestMethod]
		public void ExclusiveClosure_MultiSeed_UnionsReachability()
		{
			// Seeds: {0, 1}.  0 -> 2, 1 -> 3, X(=4) -> 2
			// 2 is tainted by X. 3 is exclusive to seed 1. Seeds always present.
			var g = BuildGraph(5, hardEdges: new[] { (0, 2), (1, 3), (4, 2) });
			AssertSetEquals(new[] { 0, 1, 3 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0), Node(g, 1) }));
		}

		[TestMethod]
		public void ExclusiveClosure_MultiSeed_OneSeedReferencesAnotherSeed_NeitherTainted()
		{
			// Seeds: {0, 1}, 0 -> 1 -> 2.  Seed 0 is "external" to seed 1 from the algo's
			// perspective only if 0 is not in seeds — but it is, so taint never fires.
			var g = BuildGraph(3, hardEdges: new[] { (0, 1), (1, 2) });
			AssertSetEquals(new[] { 0, 1, 2 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0), Node(g, 1) }));
		}

		[TestMethod]
		public void ExclusiveClosure_RuntimeEdgesRespectFlag()
		{
			// Seed: 0.   hard: 0 -> 1.   runtime: 0 -> 2, X(=3) -runtime-> 2
			// With runtime included: 2 is reachable from S but also externally — excluded.
			// With runtime excluded: 2 is not even in R; result is just {0, 1}.
			var g = BuildGraph(4,
				hardEdges:    new[] { (0, 1) },
				runtimeEdges: new[] { (0, 2), (3, 2) });

			AssertSetEquals(new[] { 0, 1 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }, includeRuntimeDependencies: true),
				"with runtime included, 2 should be tainted out");

			AssertSetEquals(new[] { 0, 1 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }, includeRuntimeDependencies: false),
				"with runtime excluded, 2 isn't in R at all");
		}

		[TestMethod]
		public void ExclusiveClosure_RuntimeEdgeTaintsOnlyWhenIncluded()
		{
			// Seed: 0. hard: 0 -> 1. runtime: X(=2) -> 1
			// With runtime included: 1 has external runtime referencer X → tainted out.
			// With runtime excluded: 1 has no external referencers (X→1 isn't a counted edge) → kept.
			var g = BuildGraph(3,
				hardEdges:    new[] { (0, 1) },
				runtimeEdges: new[] { (2, 1) });

			AssertSetEquals(new[] { 0 },    g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }, includeRuntimeDependencies: true));
			AssertSetEquals(new[] { 0, 1 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }, includeRuntimeDependencies: false));
		}

		// ------------------------------------------------------------
		// Argument validation
		// ------------------------------------------------------------

		[TestMethod]
		public void ExclusiveClosure_NullSeeds_Throws()
		{
			var g = BuildGraph(1, hardEdges: Array.Empty<(int, int)>());
			Assert.ThrowsExactly<ArgumentNullException>(() => g.GetExclusiveDependencyClosure(null!));
		}

		[TestMethod]
		public void ExclusiveClosure_SeedFromOtherGraph_Throws()
		{
			var a = BuildGraph(1, hardEdges: Array.Empty<(int, int)>());
			var b = BuildGraph(1, hardEdges: Array.Empty<(int, int)>());
			PackageNode fromB = Node(b, 0);
			Assert.ThrowsExactly<ArgumentException>(() => a.GetExclusiveDependencyClosure(new[] { fromB }));
		}

		[TestMethod]
		public void ExclusiveClosure_DuplicateSeeds_DeduplicatedSilently()
		{
			var g = BuildGraph(2, hardEdges: new[] { (0, 1) });
			var seed = Node(g, 0);
			var result = g.GetExclusiveDependencyClosure(new[] { seed, seed, seed });
			AssertSetEquals(new[] { 0, 1 }, result);
		}

		// ------------------------------------------------------------
		// Complex multi-layer graphs
		// ------------------------------------------------------------

		[TestMethod]
		public void ExclusiveClosure_MultiLayerDiamond_FullyExclusive()
		{
			// Seed: 0.  No external referencers anywhere.
			//
			//              0  (seed)
			//             / \
			//            1   2
			//             \ /
			//              3
			//              |
			//              4
			//
			var g = BuildGraph(5, hardEdges: new[]
			{
				(0, 1), (0, 2),
				(1, 3), (2, 3),
				(3, 4),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3, 4 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		[TestMethod]
		public void ExclusiveClosure_DeepChain_ExternalAtDepth3_TaintsBelowOnly()
		{
			// Seed: 0.  X(=5) injects external reach at node 3.
			//
			//   0 → 1 → 2 → 3 → 4
			//               ↑
			//               5  (external)
			//
			// 3 is the direct external-entry → tainted.
			// 4 is downstream of 3 → tainted via forward propagation.
			// 0, 1, 2 are above the entry point and stay exclusive.
			var g = BuildGraph(6, hardEdges: new[]
			{
				(0, 1), (1, 2), (2, 3), (3, 4),
				(5, 3),
			});
			AssertSetEquals(new[] { 0, 1, 2 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		[TestMethod]
		public void ExclusiveClosure_WideFanOut_OneBranchTainted()
		{
			// Seed: 0 has five children. External X(=6) points to only one (node 1).
			//
			//          0  (seed)
			//        / | | | \
			//       1  2 3 4  5
			//       ↑
			//       6  (external)
			//
			var g = BuildGraph(7, hardEdges: new[]
			{
				(0, 1), (0, 2), (0, 3), (0, 4), (0, 5),
				(6, 1),
			});
			AssertSetEquals(new[] { 0, 2, 3, 4, 5 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		[TestMethod]
		public void ExclusiveClosure_FanIn_DownstreamOfTaintedNodeAlsoTainted()
		{
			// Seed: 0.  Three intermediate nodes all converge on 4. X(=5) → 4. 4 → 6.
			//
			//             0  (seed)
			//           / | \
			//          1  2  3
			//           \ | /
			//             4 ← 5  (external)
			//             |
			//             6
			//
			// 4 is tainted (external X → 4). Taint propagates forward to 6.
			// 1, 2, 3 are upstream of 4 (their successor is tainted, but they themselves are not).
			var g = BuildGraph(7, hardEdges: new[]
			{
				(0, 1), (0, 2), (0, 3),
				(1, 4), (2, 4), (3, 4),
				(4, 6),
				(5, 4),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		[TestMethod]
		public void ExclusiveClosure_LargeSCC_NoExternalEntry_AllIncluded()
		{
			// Seed: 0.  Three-node SCC {1, 2, 3}.  Node 4 hangs off the SCC.
			//
			//          0  (seed)
			//          ↓
			//          1 ←─────┐
			//          ↓       │
			//          2       │
			//          ↓       │
			//          3 ──────┘
			//          ↓
			//          4
			//
			var g = BuildGraph(5, hardEdges: new[]
			{
				(0, 1),
				(1, 2), (2, 3), (3, 1),
				(3, 4),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3, 4 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		[TestMethod]
		public void ExclusiveClosure_LargeSCC_ExternalEnterOneMember_WholeSCCTainted()
		{
			// Seed: 0.  Same SCC as above. X(=5) enters the SCC at node 2.
			//
			//          0  (seed)
			//          ↓
			//          1 ←─────┐
			//          ↓       │
			//   5 →    2       │     (5 is external)
			//          ↓       │
			//          3 ──────┘
			//          ↓
			//          4
			//
			// 2 is tainted directly. Taint propagates 2→3→1 (around the SCC) and 3→4.
			// Everything in {1,2,3,4} is excluded; only seed 0 survives.
			var g = BuildGraph(6, hardEdges: new[]
			{
				(0, 1),
				(1, 2), (2, 3), (3, 1),
				(3, 4),
				(5, 2),
			});
			AssertSetEquals(new[] { 0 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		[TestMethod]
		public void ExclusiveClosure_ExternalPointsToMultipleDepths_OnlyDirectTargetsAndDescendantsTainted()
		{
			// Seed: 0.  X(=7) is external and points into two different branches at different depths.
			//
			//             0  (seed)
			//           / | \
			//          1  2  3
			//          |  |  |
			//          4  5  6
			//          ↑     ↑
			//          7─────┘   (external → 4 AND 6)
			//
			// Direct external entries: {4, 6}.  No forward edges out of 4 or 6, so taint doesn't propagate.
			// 5 is reached only via seed → 2 → 5, so it stays exclusive.
			// 1, 2, 3 are upstream of the entries — they themselves have no external referencers.
			var g = BuildGraph(8, hardEdges: new[]
			{
				(0, 1), (0, 2), (0, 3),
				(1, 4), (2, 5), (3, 6),
				(7, 4), (7, 6),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3, 5 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		[TestMethod]
		public void ExclusiveClosure_SelfLoop_NoExternal_NodeIncluded()
		{
			// Seed: 0.  Node 1 has a self-loop.
			//
			//          0  (seed)
			//          ↓
			//          1 ⟲   (self-loop)
			//          ↓
			//          2
			//
			// Self-loop is internal to R; not external taint.
			var g = BuildGraph(3, hardEdges: new[]
			{
				(0, 1), (1, 1), (1, 2),
			});
			AssertSetEquals(new[] { 0, 1, 2 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		[TestMethod]
		public void ExclusiveClosure_SelfLoopWithExternalReferencer_Tainted()
		{
			// Seed: 0.  Node 1 has self-loop AND external referencer 3.
			//
			//          0  (seed)
			//          ↓
			//   3 →    1 ⟲       (3 is external, 1 has self-loop)
			//          ↓
			//          2
			//
			var g = BuildGraph(4, hardEdges: new[]
			{
				(0, 1), (1, 1), (1, 2),
				(3, 1),
			});
			AssertSetEquals(new[] { 0 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		[TestMethod]
		public void ExclusiveClosure_TwoSeeds_SharedSubgraph_FullyExclusive()
		{
			// Seeds: {0, 1}.  Both depend on 2, which has its own subtree.
			//
			//   0 (seed)     1 (seed)
			//        \      /
			//         \    /
			//           2
			//          / \
			//         3   4
			//             |
			//             5
			//
			// Every consumer of {2,3,4,5} is in the seed set, so they're all exclusive.
			var g = BuildGraph(6, hardEdges: new[]
			{
				(0, 2), (1, 2),
				(2, 3), (2, 4),
				(4, 5),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3, 4, 5 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0), Node(g, 1) }));
		}

		[TestMethod]
		public void ExclusiveClosure_TwoSeeds_SharedSubgraph_PartiallyTainted()
		{
			// Seeds: {0, 1}.  Two seeds share node 4, but X(=6) also references 4.
			// 4 has its own subtree (5, 7).
			//
			//   0 (seed)     1 (seed)        6  (external)
			//      \           |             /
			//       2          3            /
			//        \        /            /
			//         \      /            /
			//            4  ←────────────
			//           / \
			//          5   7
			//
			// 4 is tainted by X. Taint propagates forward to 5 and 7.
			// 2 and 3 stay exclusive (their successor is tainted, but they themselves have no external refs).
			var g = BuildGraph(8, hardEdges: new[]
			{
				(0, 2), (1, 3),
				(2, 4), (3, 4),
				(4, 5), (4, 7),
				(6, 4),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0), Node(g, 1) }));
		}

		[TestMethod]
		public void ExclusiveClosure_TwoExternalSourcesAtDifferentDepths()
		{
			// Seed: 0.  External X(=5) at depth-2 (taints 2), external Y(=6) at depth-4 (taints 4).
			//
			//   0 (seed) → 1 → 2 → 3 → 4
			//                  ↑       ↑
			//                  5       6
			//
			// Taint from 2 propagates 2→3→4. Taint from 6→4 is redundant.
			// 1 is upstream of all entry points → exclusive. 0 is seed.
			var g = BuildGraph(7, hardEdges: new[]
			{
				(0, 1), (1, 2), (2, 3), (3, 4),
				(5, 2),
				(6, 4),
			});
			AssertSetEquals(new[] { 0, 1 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		[TestMethod]
		public void ExclusiveClosure_BackwardEdgeToSeedFromR_StillExclusive()
		{
			// Seed: 0.  R-internal cycle that loops back through seed.
			//
			//   0 (seed) → 1 → 2 → 0  (back-edge to seed)
			//                  ↓
			//                  3
			//
			// 2 → 0: predecessor 2 is in R, not external — does not taint 0 (and seeds are never tainted anyway).
			var g = BuildGraph(4, hardEdges: new[]
			{
				(0, 1), (1, 2), (2, 0),
				(2, 3),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		[TestMethod]
		public void ExclusiveClosure_NodeOutsideRPointsAtSeed_NotTainted()
		{
			// Seed: 0. External X(=3) points only at the seed; the rest of R has no external refs.
			//
			//   3 → 0 (seed) → 1 → 2
			//
			// Step 2 of the algorithm skips seeds when scanning for external referencers, so
			// nothing is tainted. 1 and 2 are exclusive; 0 is seed; result includes all of R.
			var g = BuildGraph(4, hardEdges: new[]
			{
				(0, 1), (1, 2),
				(3, 0),
			});
			AssertSetEquals(new[] { 0, 1, 2 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		[TestMethod]
		public void ExclusiveClosure_ComplexMixedTaint_DeepGraph()
		{
			// Seed: 0.  A multi-layered graph with mixed taint patterns.
			//
			//                    0  (seed)
			//                 /  |  |  \
			//                1   2  3   4
			//               /|   |  |   |
			//              5 6   7  8   9
			//              |     |  ↑   |
			//             10     11 │   12
			//                       │
			//                       13  (external)
			//
			// External referencer 13 → 8.
			// Direct taint: {8}.
			// 8 has no successors in R, so no forward propagation.
			// Expected: all of R except {8}.
			var g = BuildGraph(14, hardEdges: new[]
			{
				(0, 1), (0, 2), (0, 3), (0, 4),
				(1, 5), (1, 6),
				(2, 7),
				(3, 8),
				(4, 9),
				(5, 10),
				(7, 11),
				(9, 12),
				(13, 8),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3, 4, 5, 6, 7, 9, 10, 11, 12 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		[TestMethod]
		public void ExclusiveClosure_ExternalNodeAlsoInR_NotConsideredExternalForTaint()
		{
			// Seed: 0.  Node 3 is reachable from 0 (so 3 ∈ R) AND 3 has a forward edge to 1.
			// 3 is not a seed, but it IS in R — so an edge from 3 to 1 is NOT an "external referencer"
			// for the algorithm. 1 should stay exclusive.
			//
			//   0 (seed) → 1
			//   0         → 3
			//              ↓
			//              1   (3 → 1 is internal to R)
			//
			var g = BuildGraph(4, hardEdges: new[]
			{
				(0, 1),
				(0, 3),
				(3, 1),
			});
			AssertSetEquals(new[] { 0, 1, 3 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		[TestMethod]
		public void ExclusiveClosure_TwoIndependentTaintSources_BothPropagate()
		{
			// Seed: 0.  Two disjoint external sources X(=5) and Y(=6) taint separate subtrees.
			//
			//             0  (seed)
			//           /   \
			//          1     2
			//          |     |
			//          3     4
			//          ↑     ↑
			//          5     6
			//
			// Direct taint set: {3, 4}.  No descendants, no propagation. 1, 2 stay clean.
			var g = BuildGraph(7, hardEdges: new[]
			{
				(0, 1), (0, 2),
				(1, 3), (2, 4),
				(5, 3),
				(6, 4),
			});
			AssertSetEquals(new[] { 0, 1, 2 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		[TestMethod]
		public void ExclusiveClosure_TwoSeeds_OneSeedRescuesAnothersTaintedDep()
		{
			// Seeds: {0, 5}.  X(=6) → 2 (would normally taint 2)... but here 5 is also a seed
			// pointing at 2, and X is the only outsider. Since 6 ∉ seeds, 6 is external — and
			// 2 has 6 as an external referencer, so 2 IS tainted.
			// This test asserts the algorithm does NOT consider "seed 5 also depends on 2"
			// as somehow rescuing 2 from taint — only that the seed set is the boundary.
			//
			//   0 (seed)    5 (seed)    6  (external)
			//        \       |          /
			//         \      |         /
			//             →  2  ←
			//                |
			//                3
			//
			var g = BuildGraph(7, hardEdges: new[]
			{
				(0, 2), (5, 2),
				(2, 3),
				(6, 2),
			});
			AssertSetEquals(new[] { 0, 5 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0), Node(g, 5) }));
		}

		[TestMethod]
		public void ExclusiveClosure_HybridHardAndRuntimeEdges_RuntimeProvidesTaintPath()
		{
			// Seed: 0.  Hard chain 0 → 1 → 2.  External X(=3) reaches 2 via a RUNTIME edge.
			//
			//   0 (seed) ──hard──→ 1 ──hard──→ 2
			//                                  ↑
			//                                  ┊ runtime
			//                                  3  (external)
			//
			// With runtime included: 2 is tainted via runtime referencer.
			// With runtime excluded: 2 has no external referencers; full closure is exclusive.
			var g = BuildGraph(4,
				hardEdges:    new[] { (0, 1), (1, 2) },
				runtimeEdges: new[] { (3, 2) });

			AssertSetEquals(new[] { 0, 1 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }, includeRuntimeDependencies: true));
			AssertSetEquals(new[] { 0, 1, 2 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }, includeRuntimeDependencies: false));
		}

		[TestMethod]
		public void ExclusiveClosure_LongTaintChain_PropagatesThroughManyLevels()
		{
			// Seed: 0.  X(=8) taints node 1 at the top; taint must propagate through 6 levels.
			//
			//   0 (seed)
			//   ↓
			//   1 ← 8  (external)
			//   ↓
			//   2
			//   ↓
			//   3
			//   ↓
			//   4
			//   ↓
			//   5
			//   ↓
			//   6
			//   ↓
			//   7
			//
			// Only seed 0 survives.
			var g = BuildGraph(9, hardEdges: new[]
			{
				(0, 1),
				(1, 2), (2, 3), (3, 4), (4, 5), (5, 6), (6, 7),
				(8, 1),
			});
			AssertSetEquals(new[] { 0 }, g.GetExclusiveDependencyClosure(new[] { Node(g, 0) }));
		}

		// ------------------------------------------------------------
		// Multi-seed scenarios
		// ------------------------------------------------------------

		[TestMethod]
		public void ExclusiveClosure_ThreeSeeds_DisjointSubtrees_AllExclusive()
		{
			// Seeds: {0, 1, 2}.  Three independent subtrees, no externals.
			//
			//   0 (seed)    1 (seed)    2 (seed)
			//     ↓           ↓           ↓
			//     3           4           5
			//     ↓           ↓           ↓
			//     6           7           8
			//
			var g = BuildGraph(9, hardEdges: new[]
			{
				(0, 3), (3, 6),
				(1, 4), (4, 7),
				(2, 5), (5, 8),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3, 4, 5, 6, 7, 8 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0), Node(g, 1), Node(g, 2) }));
		}

		[TestMethod]
		public void ExclusiveClosure_ThreeSeeds_OneTaintedSubtree()
		{
			// Seeds: {0, 1, 2}.  Same shape as above, but X(=9) → 4 taints seed 1's subtree.
			//
			//   0 (seed)    1 (seed)    2 (seed)
			//     ↓           ↓           ↓
			//     3           4 ← 9       5
			//     ↓           ↓           ↓
			//     6           7           8
			//
			// 4 is tainted, propagates to 7.  Other subtrees untouched.
			var g = BuildGraph(10, hardEdges: new[]
			{
				(0, 3), (3, 6),
				(1, 4), (4, 7),
				(2, 5), (5, 8),
				(9, 4),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3, 5, 6, 8 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0), Node(g, 1), Node(g, 2) }));
		}

		[TestMethod]
		public void ExclusiveClosure_SeedsFormChain_AllSeedsInResult()
		{
			// Seeds: {0, 1, 2} all on the same chain.
			//
			//   0 (seed) → 1 (seed) → 2 (seed) → 3 → 4
			//
			// All seeds always in the result; 3 and 4 are exclusive (no external refs).
			var g = BuildGraph(5, hardEdges: new[]
			{
				(0, 1), (1, 2), (2, 3), (3, 4),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3, 4 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0), Node(g, 1), Node(g, 2) }));
		}

		[TestMethod]
		public void ExclusiveClosure_SeedsFormChain_ExternalMidChain()
		{
			// Seeds: {0, 1, 2}.  External X(=5) injects between two seeds — but it points at
			// a seed (1), so seeds-are-immune rule applies and there's no taint.
			//
			//   0 (seed) → 1 (seed) → 2 (seed) → 3 → 4
			//              ↑
			//              5  (external pointing at seed)
			//
			var g = BuildGraph(6, hardEdges: new[]
			{
				(0, 1), (1, 2), (2, 3), (3, 4),
				(5, 1),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3, 4 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0), Node(g, 1), Node(g, 2) }));
		}

		[TestMethod]
		public void ExclusiveClosure_SeedsFormCycle_AllSeedsAndDownstreamExclusive()
		{
			// Seeds: {0, 1, 2} form a cycle. Each has a leaf dependency outside the cycle.
			//
			//        0 (seed) ──→ 1 (seed)
			//           ↑              ↓
			//           └──── 2 (seed)
			//          ↓          ↓        ↓
			//          3          4        5
			//
			var g = BuildGraph(6, hardEdges: new[]
			{
				(0, 1), (1, 2), (2, 0),
				(0, 3), (1, 4), (2, 5),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3, 4, 5 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0), Node(g, 1), Node(g, 2) }));
		}

		[TestMethod]
		public void ExclusiveClosure_FourSeeds_ConvergentDependency_AllExclusive()
		{
			// Seeds: {0, 1, 2, 3}.  All four converge on shared node 4, which has its own subtree.
			//
			//   0   1   2   3   (all seeds)
			//    \  |   |  /
			//     \ |   | /
			//        4
			//        ↓
			//        5
			//        ↓
			//        6
			//
			// Every referencer of 4 is a seed → 4 is exclusive. 5 and 6 are downstream.
			var g = BuildGraph(7, hardEdges: new[]
			{
				(0, 4), (1, 4), (2, 4), (3, 4),
				(4, 5), (5, 6),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3, 4, 5, 6 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0), Node(g, 1), Node(g, 2), Node(g, 3) }));
		}

		[TestMethod]
		public void ExclusiveClosure_FourSeeds_OneNonSeedAlsoReferencesShared()
		{
			// Seeds: {0, 1, 2, 3}.  Same as above, plus non-seed X(=7) also depends on 4.
			//
			//   0   1   2   3       7  (external)
			//    \  |   |  /       /
			//     \ |   | /       /
			//        4 ← ─ ─ ─ ─ ─
			//        ↓
			//        5
			//        ↓
			//        6
			//
			// 4 has external referencer → tainted. Taint propagates to 5, 6.
			var g = BuildGraph(8, hardEdges: new[]
			{
				(0, 4), (1, 4), (2, 4), (3, 4),
				(4, 5), (5, 6),
				(7, 4),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0), Node(g, 1), Node(g, 2), Node(g, 3) }));
		}

		[TestMethod]
		public void ExclusiveClosure_TwoSeeds_OverlapAtMultipleLevels()
		{
			// Seeds: {0, 1}.  Both seeds share dependencies at TWO different layers.
			//
			//   0 (seed)         1 (seed)
			//      \              /
			//       2            3
			//        \          /
			//         \        /
			//          4 ───── 4    (both 2 and 3 depend on 4)
			//          ↓
			//          5
			//          |
			//   ─→  6 ← (both 0 and 1 also depend on 6 directly)
			//
			// To keep it simple-ish:
			//   0 → 2 → 4 → 5
			//   1 → 3 → 4
			//   0 → 6
			//   1 → 6
			//
			// All of {2, 3, 4, 5, 6} are exclusively referenced by either a seed or another exclusive node.
			var g = BuildGraph(7, hardEdges: new[]
			{
				(0, 2), (2, 4), (4, 5),
				(1, 3), (3, 4),
				(0, 6), (1, 6),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3, 4, 5, 6 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0), Node(g, 1) }));
		}

		[TestMethod]
		public void ExclusiveClosure_ThreeSeeds_MixedRootAndLeafSeeds()
		{
			// Seeds: {0, 4, 6} — a root seed, an intermediate seed, and a leaf seed.
			//
			//   0 (seed)
			//   ↓
			//   1
			//   ↓
			//   2 → 3
			//   ↓
			//   4 (seed)
			//   ↓
			//   5
			//   ↓
			//   6 (seed)     (leaf, no outgoing edges)
			//
			// R = forward-reachable from seeds = {0,1,2,3,4,5,6}. No externals. All exclusive.
			var g = BuildGraph(7, hardEdges: new[]
			{
				(0, 1), (1, 2), (2, 3), (2, 4),
				(4, 5), (5, 6),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3, 4, 5, 6 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0), Node(g, 4), Node(g, 6) }));
		}

		[TestMethod]
		public void ExclusiveClosure_TwoSeeds_ExternalTaintsOnlyOneSeedsSubtree()
		{
			// Seeds: {0, 1}.  Two independent subtrees, external X(=6) only taints seed 0's side.
			//
			//   0 (seed)       1 (seed)
			//     ↓               ↓
			//     2 ← 6           3       (6 is external)
			//     ↓               ↓
			//     4               5
			//
			// 2 directly tainted, 4 propagates. Seed 1's subtree fully clean.
			var g = BuildGraph(7, hardEdges: new[]
			{
				(0, 2), (2, 4),
				(1, 3), (3, 5),
				(6, 2),
			});
			AssertSetEquals(new[] { 0, 1, 3, 5 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0), Node(g, 1) }));
		}

		[TestMethod]
		public void ExclusiveClosure_FiveSeeds_ComplexMeshWithMultipleExternals()
		{
			// Seeds: {0, 1, 2, 3, 4}.  A dense mesh of shared deps, with two external sources.
			//
			//   0 (s)  1 (s)  2 (s)  3 (s)  4 (s)
			//    \    /  \    /  \    /  |
			//     \  /    \  /    \  /   |
			//      5        6       7    8       (shared deps)
			//       \       |      /     |
			//        \      |     /      |
			//         \     |    /       |
			//          ─→   9 ←─         10
			//               ↑             ↑
			//              11  (ext)     12 (ext)
			//
			// Edges:
			//   0→5, 1→5, 1→6, 2→6, 2→7, 3→7, 3→8, 4→8
			//   5→9, 6→9, 7→9
			//   8→10
			//   11→9, 12→10
			//
			// Direct taint: {9, 10}. No further forward propagation (both leaves).
			// 5, 6, 7, 8 each have only seed referencers → exclusive.
			var g = BuildGraph(13, hardEdges: new[]
			{
				(0, 5),
				(1, 5), (1, 6),
				(2, 6), (2, 7),
				(3, 7), (3, 8),
				(4, 8),
				(5, 9), (6, 9), (7, 9),
				(8, 10),
				(11, 9), (12, 10),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3, 4, 5, 6, 7, 8 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0), Node(g, 1), Node(g, 2), Node(g, 3), Node(g, 4) }));
		}

		[TestMethod]
		public void ExclusiveClosure_TwoSeeds_OneSeedDependsOnOtherSeed()
		{
			// Seeds: {0, 1}.  Seed 0 depends on seed 1.
			//
			//   0 (seed) → 1 (seed) → 2 → 3
			//
			// Both seeds always present; 2 and 3 are exclusive.
			var g = BuildGraph(4, hardEdges: new[]
			{
				(0, 1), (1, 2), (2, 3),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0), Node(g, 1) }));
		}

		[TestMethod]
		public void ExclusiveClosure_TwoSeeds_SharedCycle_NoExternal()
		{
			// Seeds: {0, 1}.  Both reach into a shared 3-node cycle {2,3,4}.
			//
			//   0 (seed)     1 (seed)
			//      \           /
			//       \         /
			//        ↓       ↓
			//          2 ────────────┐
			//          ↓             │
			//          3             │
			//          ↓             │
			//          4 ────────────┘
			//          ↓
			//          5
			//
			// Cycle referencers are seeds-only or cycle-internal → all exclusive.
			var g = BuildGraph(6, hardEdges: new[]
			{
				(0, 2), (1, 2),
				(2, 3), (3, 4), (4, 2),
				(4, 5),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3, 4, 5 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0), Node(g, 1) }));
		}

		[TestMethod]
		public void ExclusiveClosure_ThreeSeeds_NestedSharedDependencies()
		{
			// Seeds: {0, 1, 2}.  Each seed has its own primary dep; all three share a deeper dep.
			//
			//   0 (s)   1 (s)   2 (s)
			//    ↓       ↓       ↓
			//    3       4       5     (each seed → its own primary)
			//     \      |      /
			//      \     |     /
			//        →   6   ←        (all three primaries → common 6)
			//            ↓
			//            7
			//            ↓
			//            8
			//
			// 6's referencers are {3,4,5}, all of which are R-only (no external refs themselves).
			// So 6, 7, 8 all stay exclusive.
			var g = BuildGraph(9, hardEdges: new[]
			{
				(0, 3), (1, 4), (2, 5),
				(3, 6), (4, 6), (5, 6),
				(6, 7), (7, 8),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3, 4, 5, 6, 7, 8 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0), Node(g, 1), Node(g, 2) }));
		}

		[TestMethod]
		public void ExclusiveClosure_ThreeSeeds_NestedSharedDeps_DeepExternalEntry()
		{
			// Same shape as above, but X(=9) → 7 introduces external reach deep in the shared subtree.
			//
			//   0 (s)   1 (s)   2 (s)
			//    ↓       ↓       ↓
			//    3       4       5
			//     \      |      /
			//      \     |     /
			//        →   6   ←
			//            ↓
			//            7 ← 9  (external)
			//            ↓
			//            8
			//
			// Direct taint: {7}. Forward propagation: {8}.
			// 6 is upstream of 7 — its successor is tainted but it has no external referencer of its own.
			var g = BuildGraph(10, hardEdges: new[]
			{
				(0, 3), (1, 4), (2, 5),
				(3, 6), (4, 6), (5, 6),
				(6, 7), (7, 8),
				(9, 7),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3, 4, 5, 6 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0), Node(g, 1), Node(g, 2) }));
		}

		[TestMethod]
		public void ExclusiveClosure_AllSeedsAreLeaves_NoDependencies()
		{
			// Seeds: {0, 1, 2, 3}.  None has outgoing edges. Some have external referencers.
			//
			//   4 → 0 (seed)         (external referencer of seed)
			//       1 (seed)         (no edges at all)
			//       2 (seed)         (no edges at all)
			//   5 → 3 (seed)         (external referencer of seed)
			//
			// Seeds are always in the result regardless of external referencers.
			// R = seeds (no successors).  Nothing to taint.
			var g = BuildGraph(6, hardEdges: new[]
			{
				(4, 0), (5, 3),
			});
			AssertSetEquals(new[] { 0, 1, 2, 3 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0), Node(g, 1), Node(g, 2), Node(g, 3) }));
		}

		[TestMethod]
		public void ExclusiveClosure_TwoSeeds_BridgeNode_TaintCascadeAcrossSeeds()
		{
			// Seeds: {0, 1}.  A "bridge" pattern: seed 0's tainted subtree feeds into seed 1's deps.
			//
			//   0 (seed)
			//    ↓
			//    2 ← 5  (external)
			//    ↓
			//    3
			//    ↓
			//    1 (seed)   ← (3 → 1, but 1 is a seed, so taint stops here)
			//    ↓
			//    4
			//
			// 2 directly tainted, propagates 2→3.  3 → 1: 1 is a seed, never tainted.
			// 4 has only seed 1 as referencer → exclusive.
			var g = BuildGraph(6, hardEdges: new[]
			{
				(0, 2), (2, 3), (3, 1), (1, 4),
				(5, 2),
			});
			AssertSetEquals(new[] { 0, 1, 4 },
				g.GetExclusiveDependencyClosure(new[] { Node(g, 0), Node(g, 1) }));
		}
	}
}
